#!/bin/bash
SCRIPT_DIR=$( cd -- "$( dirname -- "${BASH_SOURCE[0]}" )" &> /dev/null && pwd )
"$SCRIPT_DIR"/Linux/Engine/Binaries/Linux/BuildStorageTool "$@" 2>&1 &