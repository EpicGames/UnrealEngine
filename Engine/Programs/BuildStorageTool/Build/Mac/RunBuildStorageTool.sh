#!/bin/bash
SCRIPT_DIR=$( cd -- "$( dirname -- "${BASH_SOURCE[0]}" )" &> /dev/null && pwd )
open "$SCRIPT_DIR"/Mac/BuildStorageTool.app --args "$@"