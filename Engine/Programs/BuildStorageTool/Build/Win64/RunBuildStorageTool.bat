START "" "%~dp0Windows\Engine\Binaries\Win64\BuildStorageTool.exe" %*