
[/Script/Engine.NetworkSettings]
n.VerifyPeer=true

[StudioTelemetry.Provider.EpicStudio]
Enabled=True

[StudioTelemetry.Provider.Horde.Dev]
Enabled=False

[StudioTelemetry.Provider.Horde]
Enabled=False