[Staging]
+AllowedDirectories=Engine/Content/Editor/Slate
+AllowedDirectories=Engine/Content/Slate
+AllowedConfigFiles=Engine/Config/BaseEditorPerProjectUserSettings.ini
+AllowedConfigFiles=BuildStorageTool/Config/DeployId.ini

[/Script/UnrealEd.ProjectPackagingSettings]
; Reset the ini section deny list to allow us to stage the storage server configurations for this tool
IniSectionDenylist=